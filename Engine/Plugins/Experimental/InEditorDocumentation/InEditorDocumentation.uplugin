{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "In-Editor Documentation",
	"Description": "Navigate a configured tutorial within the Unreal Editor as you explore a project.",
	"Category": "Learning",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "InEditorDocumentation",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "WebBrowserWidget",
			"Enabled": true
		}
	]
}