// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InEditorDocumentation : ModuleRules
{
	public InEditorDocumentation(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
                "Projects",
                "InputCore",
                "EditorFramework",
                "UnrealEd",
                "CoreUObject",
                "Engine",
                "Slate",
                "SlateCore",
                "Blutility",
                "UMG",
                "WebBrowser",
				"ToolMenus",
				"HTTP",
				"Json",
				"JsonUtilities",
				"LevelEditor",
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
