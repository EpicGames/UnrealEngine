// Copyright Epic Games, Inc. All Rights Reserved.


#include "CommandsStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Slate/SlateGameResources.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr<FSlateStyleSet> FCommandsStyle::StyleInstance = nullptr;

void FCommandsStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FCommandsStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

void FCommandsStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

const ISlateStyle& FCommandsStyle::Get()
{
	// TODO: insert return statement here
	return *StyleInstance;
}

FName FCommandsStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("CommandsStyle"));
	return StyleSetName;
}

const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon64x64(64.0f, 64.0f);
const FVector2D SearchIcon40x40(40.0f, 40.0f);

TSharedRef<class FSlateStyleSet> FCommandsStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShareable(new FSlateStyleSet("CommandsStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("InEditorDocumentation")->GetBaseDir() / TEXT("Resources"));

	Style->Set("InEditorDocumentation.OpenTutorial", new IMAGE_BRUSH(TEXT("EditorTutorial_64x"), Icon64x64));
	Style->Set("InEditorDocumentation.OpenSearch", new IMAGE_BRUSH(TEXT("Search_40x"), SearchIcon40x40));

	return Style;
}
