// Copyright Epic Games, Inc. All Rights Reserved.


#include "DocumentationCommands.h"

#define LOCTEXT_NAMESPACE "FInEditorDocumentationModule"

void FDocumentationCommands::RegisterCommands()
{
	// Command to bring up a documentation window
	UI_COMMAND(OpenTutorial, "Open Tutorial", "Open Tutorial", EUserInterfaceActionType::Button, FInputChord());

	// Command to search Edc for information
	UI_COMMAND(OpenSearch, "Search Epic Developer Community", "Search the Epic Developer Community", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE