// Copyright Epic Games, Inc. All Rights Reserved.

#include "InEditorDocumentation.h"

#include "CoreMinimal.h"
#include "Components/SlateWrapperTypes.h"
#include "DocumentationCommands.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HttpModule.h"
#include "Http.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "InEditorDocumentationSettings.h"
#include "Interfaces/IHttpResponse.h"
#include "Kismet/KismetSystemLibrary.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Misc/FileHelper.h"
#include "Styling/AppStyle.h"
#include "Selection.h"
#include "SWebBrowser.h"
#include "ToolMenus.h"
#include "WebBrowserModule.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"

DEFINE_LOG_CATEGORY(LogDocumentation);

static const FName TutorialTabName("Tutorial");
static const FName SearchTabName("EDCSearch");

#define LOCTEXT_NAMESPACE "FInEditorDocumentationModule"

void FInEditorDocumentationModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	// Slate Style for the OpenDocumentationWindow button
	FCommandsStyle::Initialize();
	FCommandsStyle::ReloadTextures();

	// UI Commands Init
	FDocumentationCommands::Register();

	// Initialize the commands
	PluginCommands = MakeShareable(new FUICommandList);

	check(PluginCommands.IsValid());

	// Map the OpenDocumentationWindow UI Command to the OnOpenDocumentationWindowClicked Callback
	PluginCommands->MapAction(
		FDocumentationCommands::Get().OpenTutorial,
		FExecuteAction::CreateRaw(this, &FInEditorDocumentationModule::OnToggleTutorialClicked)
	);

	// Map the SearchEdc UI Command to the OnSearchEdcClicked Callback
	PluginCommands->MapAction(
		FDocumentationCommands::Get().OpenSearch,
		FExecuteAction::CreateRaw(this, &FInEditorDocumentationModule::OnSearchClicked)
	);
	
	// Ensure that necessary modules/plugins are registered
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>(TEXT("Settings"));

	// Register configuration settings for this plugin
	ModuleConfigSettings = SettingsModule.RegisterSettings(
		SettingsContainerName,
		SettingsCategoryName,
		SettingsSectionName,
		DisplayName,
		Description,
		GetMutableDefault<UInEditorDocumentationSettings>()
	);

	// If the configuration settings are modified, handle this
	if (ModuleConfigSettings.IsValid())
	{
		ModuleConfigSettings->OnModified().BindRaw(this, &FInEditorDocumentationModule::HandleSettingsModified);
	}

	// Check the configuration settings to determine whether (experimental) search is enabled
	const UInEditorDocumentationSettings* Settings = GetDefault<UInEditorDocumentationSettings>();
	if (Settings)
	{
		bShowSearch = Settings->bEnableEdcSearch;
	}

	// Register the documentation button in the level editor toolbar
	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FInEditorDocumentationModule::RegisterMenus));

	// Register the edc search web browser
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(SearchTabName, FOnSpawnTab::CreateRaw(this, &FInEditorDocumentationModule::OnSpawnSearchTab))
		.SetDisplayName(LOCTEXT("InEditorSearchTabName", "EDC Search"))
		.SetMenuType(ETabSpawnerMenuType::Hidden);

	// Extend the level editor layout so that the web browsers appear as docked slate tabs
	ExtendLevelEditorLayout();
}

void FInEditorDocumentationModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	// Unregister module configuration settings
	ISettingsModule& SettingsModule = FModuleManager::GetModuleChecked<ISettingsModule>(TEXT("Settings"));
	SettingsModule.UnregisterSettings(SettingsContainerName, SettingsCategoryName, SettingsSectionName);
}

void FInEditorDocumentationModule::OnToggleTutorialClicked()
{
	// Manage this tab through the Level Editor Tab Manager, not the global tab manager
	const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

	// Access the level editor tab manager
	const TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

	// Try to find a live tab for documentation/tutorial through the tab manager
	TSharedPtr<SDockTab> FindTabResult = LevelEditorTabManager->FindExistingLiveTab(TutorialTabName);

	if (FindTabResult.IsValid() && TutorialWebBrowser.IsValid())
	{
		// If tab is open, try to close it
		FindTabResult->RequestCloseTab();
		// Delete the reference to TutorialWebBrowser
		TutorialWebBrowser.Reset();
	}
	else
	{
		// If either is not valid, reopen instead
		LevelEditorTabManager->TryInvokeTab(TutorialTabName);
	}
}

void FInEditorDocumentationModule::OnSearchClicked()
{

	// Try to find a live tab for documentation/tutorial through the tab manager
	TSharedPtr<SDockTab> FindTabResult = FGlobalTabmanager::Get()->FindExistingLiveTab(SearchTabName);

	if (FindTabResult.IsValid() && SearchWebBrowser.IsValid())
	{
		// If the tab and web browser are valid, try to change the url
		FString Url = ConstructSearchUrl();
		SearchWebBrowser->LoadURL(Url);
	}
	else
	{
		// If either is not valid, reopen instead
		FGlobalTabmanager::Get()->TryInvokeTab(SearchTabName);
	}
}

TObjectPtr<AActor> FInEditorDocumentationModule::GetFirstSelectedActor()
{
	TArray<TObjectPtr<AActor>> SelectedActors;

	// Access the Unreal Editor global object
	if (GEditor)
	{
		// Iterate through all selected actors in the viewport
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			// Cast to Actor and add to array
			TObjectPtr<AActor> AsActor = Cast<AActor>(*It);
			if (AsActor != nullptr)
			{
				SelectedActors.Add(AsActor);
			}
		}
	}

	// If there are any, return the first one
	if (SelectedActors.Num() > 0)
	{
		return SelectedActors[0];
	}

	return nullptr;
}

void FInEditorDocumentationModule::RegisterMenus()
{
	// Owner will be used for cleanup in call to UToolMenus::UnregisterOwner
	FToolMenuOwnerScoped OwnerScoped(this);
	{
		// Get the UI menu we wish to add a button to
		UToolMenu* ToolbarMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.PlayToolBar");
		{
			// Get the section within the toolbar menu
			FToolMenuSection& Section = ToolbarMenu->FindOrAddSection("PluginTools");
			{
				// Add our OpenDocumentationWindow UI Command to this menu
				FToolMenuEntry& OpenDocEntry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(FDocumentationCommands::Get().OpenTutorial));
				OpenDocEntry.SetCommandList(PluginCommands);

				// Add our SearchEdc UI Command to this menu
				const FToolMenuEntry& Entry = FToolMenuEntry::InitMenuEntryWithCommandList(FDocumentationCommands::Get().OpenSearch, PluginCommands, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), NAME_None, TOptional<FName>(FName("SearchEdc")));
				FToolMenuEntry& SearchEdcEntry = Section.AddEntry(Entry);

				// Alter the visibility based on the search settings
				SearchEdcEntry.Visibility = TAttribute<EVisibility>::CreateLambda([this]()
					{
						return bShowSearch ? EVisibility::Visible : EVisibility::Hidden;
					});

				SearchEntry = &SearchEdcEntry;
			}
		}
	}
}

void FInEditorDocumentationModule::ExtendLevelEditorLayout()
{
	// Ensure that the Level Editor Module is loaded
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	LevelEditorModule.OnRegisterTabs().AddLambda([this](TSharedPtr<FTabManager> TabManager)
		{
			// Register the tutorial tab spawner
			TabManager->RegisterTabSpawner(TutorialTabName, FOnSpawnTab::CreateRaw(this, &FInEditorDocumentationModule::OnSpawnTutorialTab))
				.SetDisplayName(LOCTEXT("InEditorTutorialTabName", "Tutorial"))
				.SetMenuType(ETabSpawnerMenuType::Hidden);
		});

	// Extend the layout so the documentation/tutorial can dock next to the level editor viewport
	LevelEditorModule.OnRegisterLayoutExtensions().AddLambda([](FLayoutExtender& Extender)
		{
			Extender.ExtendLayout(LevelEditorTabIds::LevelEditorViewport, ELayoutExtensionPosition::Above, FTabManager::FTab(TutorialTabName, ETabState::OpenedTab));
		});

	// Once the level editor is created, try to open the documentation window
	LevelEditorModule.OnLevelEditorCreated().AddLambda([this](TSharedPtr<ILevelEditor> LevelEditorInstance) 
		{
			float ReloadDelay = 3.5f;
			LevelEditorInstance->GetTabManager()->TryInvokeTab(TutorialTabName);
			FTSTicker::GetCoreTicker().AddTicker(
				FTickerDelegate::CreateLambda([this](float)
					{
						if (this->TutorialWebBrowser.IsValid())
						{
							FString TutorialUrl = this->GetTutorialUrl();
							UE_LOGF(LogDocumentation, Display, "Force tutorial url reload after delay: %ls", *TutorialUrl);
							this->TutorialWebBrowser->LoadURL(TutorialUrl);
						}
						return false;
					}),
				ReloadDelay
			);
		});
}

FString FInEditorDocumentationModule::GetTutorialUrl()
{
	// Default url to start
	FString DefaultUrl = TEXT("https://dev.epicgames.com/documentation/unreal-engine/");
	FString Url = DefaultUrl;

	// Access the configuration settings for this plugin
	UInEditorDocumentationSettings* AsModuleSettings;
	TWeakObjectPtr<UObject> ModuleSettings = ModuleConfigSettings->GetSettingsObject();

	// Default to the url set in configuration
	if (ModuleSettings.IsValid())
	{
		AsModuleSettings = Cast<UInEditorDocumentationSettings>(ModuleSettings);
		if (AsModuleSettings != nullptr)
		{
			Url = AsModuleSettings->TutorialUrl;
		}
	}

	if (!Url.StartsWith(TEXT("https://dev.epicgames.com/")))
	{
		UE_LOGF(LogDocumentation, Display, "URL must belong to dev.epicgames.com");
		return DefaultUrl;
	}
	return Url;
}

void FInEditorDocumentationModule::HandleLoadErrorContent()
{
	UE_LOGF(LogDocumentation, Display, "Page load error")
	FString NotFoundContentPath = FPaths::ProjectDir() / TEXT("Plugins/InEditorDocumentation/Resources/page-not-found.html");
	FString Content;
	if (FFileHelper::LoadFileToString(Content, *NotFoundContentPath) && TutorialWebBrowser.IsValid())
	{
		TutorialWebBrowser->StopLoad();
		TutorialWebBrowser->LoadString(Content, TEXT("404"));
	}
}


FString FInEditorDocumentationModule::ConstructSearchUrl()
{
	// Default url to start
	FString DefaultUrl = TEXT("https://dev.epicgames.com/community/search?types=document&application=unreal_engine&page=1&sort_by=relevancy&published_at_range=any_time");
	FString Url = DefaultUrl;

	// Access the configuration settings for this plugin
	UInEditorDocumentationSettings* AsModuleSettings;
	TWeakObjectPtr<UObject> ModuleSettings = ModuleConfigSettings->GetSettingsObject();

	// Obtain any selected actors from the viewport
	TObjectPtr<AActor> SelectedActor = GetFirstSelectedActor();

	// If any actors are selected in the viewport, try to find preconfigured documentation or search the edc
	if (SelectedActor)
	{
		// Get the class name of the selected actor from viewport
		TObjectPtr<UClass> FirstActorClass = SelectedActor->GetClass();
		FString ActorClassName = FirstActorClass->GetName().ToLower();

		bool bConfigUrlFound = false;
		
		if (ModuleSettings.IsValid())
		{
			// Search for the name of the actor's class in preconfigured pages
			AsModuleSettings = Cast<UInEditorDocumentationSettings>(ModuleSettings);
			if (AsModuleSettings != nullptr)
			{
				if (AsModuleSettings->DocumentationPages.Contains(ActorClassName))
				{
					Url = AsModuleSettings->DocumentationPages[ActorClassName];
					bConfigUrlFound = true;
					UE_LOGF(LogDocumentation, Display, "Found class [%ls] with url [%ls]", *ActorClassName, *Url)
				}
			}
		}

		// If the actor was not found in configuration, search the edc
		if (!bConfigUrlFound)
		{
			// Construct the search url
			FString BaseUrl = TEXT("https://dev.epicgames.com/community/search");
			FString Options = TEXT("&types=document&application=unreal_engine&page=1&sort_by=relevancy&published_at_range=any_time");

			FStringFormatOrderedArguments UrlArguments;
			UrlArguments.Add(*BaseUrl);
			UrlArguments.Add(*ActorClassName);
			UrlArguments.Add(*Options);

			Url = FString::Format(TEXT("{0}?query={1}{2}"), UrlArguments);
			UE_LOGF(LogDocumentation, Display, "Searching edc for actor [%ls] with url [%ls]", *ActorClassName, *Url);
		}
	}

	if (!Url.StartsWith(TEXT("https://dev.epicgames.com/")))
	{
		UE_LOGF(LogDocumentation, Display, "URL must belong to dev.epicgames.com");
		return DefaultUrl;
	}
	return Url;
}

TSharedRef<class SDockTab> FInEditorDocumentationModule::OnSpawnTutorialTab(const FSpawnTabArgs& SpawnTabArgs)
{
	IWebBrowserModule& WebBrowserModule = IWebBrowserModule::Get();
	if (!IWebBrowserModule::IsAvailable() || !WebBrowserModule.IsWebModuleAvailable())
	{
		UE_LOGF(LogDocumentation, Display, "WebBrowserModule isn't available! Can't display Tutorial tab web content.");
		return SNew(SDockTab)
			.TabRole(ETabRole::DocumentTab)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(0, 32, 0, 0))
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
							.Text(FText::FromString("Unable to display Tutorial content"))
							.Justification(ETextJustify::Center)
					]
			];
	}

	// Obtain the tutorial url
	FString Url = GetTutorialUrl();

	// Create web browser and apply relevant settings
	SAssignNew(TutorialWebBrowser, SWebBrowser)
		.InitialURL(Url)
		.ShowControls(true)
		.ShowAddressBar(false)
		.ShowErrorMessage(true)
		.SupportsTransparency(true)
		.PopupMenuMethod(EPopupMethod::UseCurrentWindow)
		.OnBeforeNavigation(SWebBrowser::FOnBeforeBrowse::CreateRaw(this, &FInEditorDocumentationModule::HandleEdcNavigation))
		.OnLoadError(FSimpleDelegate::CreateRaw(this, &FInEditorDocumentationModule::HandleLoadErrorContent))
		.OnLoadCompleted(FSimpleDelegate::CreateRaw(this, &FInEditorDocumentationModule::HandleTutorialLoadCompleted));
	

	return SNew(SDockTab)
		.TabRole(ETabRole::DocumentTab)
		.OnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FInEditorDocumentationModule::HandleTutorialTabClosed))
		[
			TutorialWebBrowser.ToSharedRef()
		];
}

TSharedRef<SDockTab> FInEditorDocumentationModule::OnSpawnSearchTab(const FSpawnTabArgs& SpawnTabArgs)
{
	IWebBrowserModule& WebBrowserModule = IWebBrowserModule::Get();
	if (!IWebBrowserModule::IsAvailable() || !WebBrowserModule.IsWebModuleAvailable())
	{
		UE_LOGF(LogDocumentation, Display, "WebBrowserModule isn't available! Can't display Search tab web content.");
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(0, 32, 0, 0))
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
							.Text(FText::FromString("Unable to display Search content"))
							.Justification(ETextJustify::Center)
					]
			];
	}

	// Obtain the default search url or construct a custom one based on the selected actor in the viewport
	FString Url = ConstructSearchUrl();

	// Web browser is created and settings applied here
	SAssignNew(SearchWebBrowser, SWebBrowser)
		.InitialURL(Url)
		.ShowControls(true)
		.ShowAddressBar(false)
		.PopupMenuMethod(EPopupMethod::UseCurrentWindow)
		.OnBeforeNavigation(SWebBrowser::FOnBeforeBrowse::CreateRaw(this, &FInEditorDocumentationModule::HandleEdcNavigation))
		.OnLoadCompleted(FSimpleDelegate::CreateRaw(this, &FInEditorDocumentationModule::HandleSearchLoadCompleted));

	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SearchWebBrowser.ToSharedRef()
		];
}

void FInEditorDocumentationModule::HandleTutorialTabClosed(TSharedRef<SDockTab> ClosedTab)
{
	// If the user has not manually acknowledged and dismissed the tutorial closed tab,
	// Display a toast so that the user has to acknowledge it
	if (!bUserDismissedTutorialNotification && !TutorialToast.IsValid())
	{
		DisplayTutorialToastNotification();
	}
}

bool FInEditorDocumentationModule::HandleEdcNavigation(const FString& NavUrl, const FWebNavigationRequest& Nav)
{
	// Only allow links within the EDC in editor
	if (NavUrl.StartsWith(TEXT("https://dev.epicgames.com/")))
	{
		return false;
	}
	else
	{
		// Deny non-EDC links
		UE_LOGF(LogDocumentation, Display, "URLs external to dev.epicgames.com are disabled");
		return true;
	}
}

void FInEditorDocumentationModule::HandleTutorialLoadCompleted()
{
	if (this->TutorialWebBrowser.IsValid())
	{
		// Disable certain elements for Edc site
		// This allows for more viewing space within the editor
		this->TutorialWebBrowser->ExecuteJavascript(TEXT(
			// Disable site navigation and TOC
			"document.querySelectorAll('.site-nav').forEach(function(el) { el.style.display = 'none'; });"
			// Disable header which also contains search bar
			"document.querySelectorAll('.site-header').forEach(function(el) { el.style.display = 'none'; });"
			// Disable breadcrumb menu that appears on each page
			"document.querySelectorAll('.breadcrumb').forEach(function(el) { el.style.display = 'none'; });"
			// Disable mobile TOC that only appears when window size is narrow
			"document.querySelectorAll('.is-table-of-contents').forEach(function(el) { el.style.display = 'none'; });"
			// Scale down the overall rem size
			"document.documentElement.style.fontSize = '13px';"
			// Change page styles for scrollbars and hide link popups
			"(function() { const styleElement = document.createElement('style');\n\
			styleElement.textContent = '::-webkit-scrollbar { width:8px; } ::-webkit-scrollbar-thumb { border-radius:8px;background-color:#575757; } ::-webkit-scrollbar-thumb:hover { background-color:#808080; } .glossary-term-popup-detail-wrapper { display: none; }';\n\
            document.head.appendChild(styleElement); })();"
		));
	}
}

void FInEditorDocumentationModule::HandleSearchLoadCompleted()
{
	if (this->SearchWebBrowser.IsValid())
	{
		// Disable certain elements for Edc site
		// This allows for more viewing space within the editor
		this->SearchWebBrowser->ExecuteJavascript(TEXT(
			// Disable site navigation and TOC
			"document.querySelectorAll('.site-nav').forEach(function(el) { el.style.display = 'none'; });"
			// Disable header which also contains search bar
			"document.querySelectorAll('.site-header').forEach(function(el) { el.style.display = 'none'; });"
			// Disable breadcrumb menu that appears on each page
			"document.querySelectorAll('.breadcrumb').forEach(function(el) { el.style.display = 'none'; });"
			// Disable search filtering menu
			"document.querySelectorAll('.col-filters-wrapper').forEach(function(el) { el.style.display = 'none'; });"
			// Disable mobile TOC that only appears when window size is narrow
			"document.querySelectorAll('.is-table-of-contents').forEach(function(el) { el.style.display = 'none'; });"
			// Scale down the overall rem size
			"document.documentElement.style.fontSize = '13px';"
			// Change page styles for scrollbars and hide link popups
			"(function() { const styleElement = document.createElement('style');\n\
			styleElement.textContent = '::-webkit-scrollbar { width:8px; } ::-webkit-scrollbar-thumb { border-radius:8px;background-color:#575757; } ::-webkit-scrollbar-thumb:hover { background-color:#808080; } .glossary-term-popup-detail-wrapper { display: none; }';\n\
            document.head.appendChild(styleElement); })();"
		));
	}
}

void FInEditorDocumentationModule::DisplayTutorialToastNotification()
{
	// Construct toast notification
	FNotificationInfo Notification(FText::FromString(TEXT("Tutorial Closed")));
	Notification.SubText = FText::FromString(TEXT("To re-open the tutorial, navigate to the Level Editor toolbar and click the Open Tutorial button."));
	Notification.ExpireDuration = 0.0f;
	Notification.bFireAndForget = false;
	Notification.bUseThrobber = true;
	Notification.Image = FAppStyle::GetBrush("Icons.Warning");
	Notification.CheckBoxText = FText::FromString(TEXT("Acknowledged"));
	Notification.CheckBoxStateChanged.BindRaw(this, &FInEditorDocumentationModule::HandleTutorialToastCheckboxStateChanged);

	// Add the toast notification to the manager to display
	TutorialToast = FSlateNotificationManager::Get().AddNotification(Notification);

	if (TutorialToast.IsValid())
	{
		UE_LOGF(LogDocumentation, Display, "Setting toast completion state to pending");
		TutorialToast.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
	else
	{
		UE_LOGF(LogDocumentation, Display, "Toast invalid");
	}
}

void FInEditorDocumentationModule::HandleTutorialToastCheckboxStateChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Checked)
	{
		UE_LOGF(LogDocumentation, Display, "Checkbox state changed to checked");

		// If the user has checked the state, record it so the toast does not appear again
		this->bUserDismissedTutorialNotification = true;

		// If the toast is still valid, set it to complete and remove
		if (TutorialToast.IsValid())
		{
			UE_LOGF(LogDocumentation, Display, "Toast valid, attempting to dismiss");
			TutorialToast.Pin()->SetCompletionState(SNotificationItem::CS_Success);
			TutorialToast.Pin()->ExpireAndFadeout();
		}
		else
		{
			UE_LOGF(LogDocumentation, Display, "Toast invalid");
		}
	}
	else
	{
		UE_LOGF(LogDocumentation, Display, "Unrecognized checkbox state change")
	}
}

bool FInEditorDocumentationModule::HandleSettingsModified()
{
	const UInEditorDocumentationSettings* Settings = GetDefault<UInEditorDocumentationSettings>();
	if (Settings->bEnableEdcSearch)
	{
		UE_LOGF(LogDocumentation, Display, "Edc search enabled");

		// Make search visible
		if (SearchEntry)
		{
			bShowSearch = true;
			UToolMenus::Get()->RefreshMenuWidget(FName(TEXT("SearchEdc")));
		}
	}
	else
	{
		// Make invisible
		if (SearchEntry)
		{
			UE_LOGF(LogDocumentation, Display, "Found search entry");
			bShowSearch = false;

			// Manage this tab through the Level Editor Tab Manager, not the global tab manager
			const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

			// Access the level editor tab manager
			const TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

			// Try to find a live tab for documentation/tutorial through the tab manager
			TSharedPtr<SDockTab> FindTabResult = LevelEditorTabManager->FindExistingLiveTab(SearchTabName);

			// Try to close the search tab
			if (FindTabResult.IsValid() && FindTabResult->RequestCloseTab())
			{
				UE_LOGF(LogDocumentation, Display, "Closed search tab")
			}
			else
			{
				UE_LOGF(LogDocumentation, Display, "Could not close search tab")
			}

			UToolMenus::Get()->RefreshMenuWidget(FName(TEXT("SearchEdc")));
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FInEditorDocumentationModule, InEditorDocumentation)