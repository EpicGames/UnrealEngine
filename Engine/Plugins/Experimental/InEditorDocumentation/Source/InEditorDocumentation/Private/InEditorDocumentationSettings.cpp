// Copyright Epic Games, Inc. All Rights Reserved.


#include "InEditorDocumentationSettings.h"

#if WITH_EDITOR
void UInEditorDocumentationSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	SaveConfig();
}
#endif // WITH_EDITOR
