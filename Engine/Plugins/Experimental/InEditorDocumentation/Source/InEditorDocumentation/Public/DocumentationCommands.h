// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommandsStyle.h"
#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

/**
 * 
 */
class INEDITORDOCUMENTATION_API FDocumentationCommands : public TCommands<FDocumentationCommands>
{
public:
	FDocumentationCommands() : TCommands<FDocumentationCommands>(TEXT("InEditorDocumentation"), NSLOCTEXT("Contexts", "InEditorDocumentation", "InEditorDocumentation Plugin"), NAME_None, FCommandsStyle::GetStyleSetName())
	{
	}

	// TCommands<...> implementation
	virtual void RegisterCommands() override;
	// TCommands<...> implementation

public:
	TSharedPtr<FUICommandInfo> OpenTutorial;
	TSharedPtr<FUICommandInfo> OpenSearch;
};