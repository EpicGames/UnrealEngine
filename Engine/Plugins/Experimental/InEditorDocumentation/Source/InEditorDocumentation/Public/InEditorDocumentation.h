// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateTypes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDocumentation, Display, All);

class AActor;
class FUICommandList;
class FJsonObject;
class ISettingsSection;
class SDockTab;
class SNotificationItem;
class SWebBrowser;

struct FToolMenuEntry;
struct FWebNavigationRequest;

typedef TSharedPtr<ISettingsSection> ISettingsSectionPtr;

class FInEditorDocumentationModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	/** IModuleInterface implementation */

private:

	/* Configuration Section Settings */
	ISettingsSectionPtr ModuleConfigSettings;
	const FName SettingsContainerName = "Editor";
	const FName SettingsCategoryName = "Plugins";
	const FName SettingsSectionName = "In-Editor Documentation";
	const FText DisplayName = FText::FromString("In-Editor Documentation");
	const FText Description = FText::FromString("Settings for documentation in the Unreal Editor");

	// Toast notification
	TWeakPtr<SNotificationItem> TutorialToast;

	// Has the user manually dismissed the tutorial toast?
	bool bUserDismissedTutorialNotification = false;

	// Is Edc search enabled?
	bool bShowSearch = true;

	// Ui commands
	TSharedPtr<FUICommandList> PluginCommands;

	// Web browser
	TSharedPtr<SWebBrowser> TutorialWebBrowser;

	// Search web browser
	TSharedPtr<SWebBrowser> SearchWebBrowser;

	// Edc Search results
	TSharedPtr<FJsonObject> SearchResults;

	// Search entry in the Level Editor menu
	FToolMenuEntry* SearchEntry;

	/// <summary>
	/// Callback - Invoked as a response to UI_COMMAND(OpenDocumentationWindow,...)
	/// </summary>
	void OnToggleTutorialClicked();

	/// <summary>
	/// Callback - Invoked as a response to UI_COMMAND(SearchEdc,...)
	/// </summary>
	void OnSearchClicked();

	/// <summary>
	/// Get an array of all actors currently selected in the Unreal Editor viewport
	/// </summary>
	/// <returns></returns>
	static TObjectPtr<AActor> GetFirstSelectedActor();

	/// <summary>
	/// Register UI Menus
	/// </summary>
	void RegisterMenus();

	/// <summary>
	/// Extend the level editor layout
	/// </summary>
	void ExtendLevelEditorLayout();

	/// <summary>
	/// Get the url to navigate to
	/// </summary>
	/// <returns></returns>
	FString GetTutorialUrl();

	/// <summary>
	/// Construct the url for the search edc
	/// </summary>
	/// <returns></returns>
	FString ConstructSearchUrl();

	/// <summary>
	/// Callback - Invoked as a response to TryInvokeTab() for OpenDocumentationWindow
	/// </summary>
	/// <param name="SpawnTabArgs"></param>
	/// <returns></returns>
	TSharedRef<SDockTab> OnSpawnTutorialTab(const class FSpawnTabArgs& SpawnTabArgs);

	/// <summary>
	/// Callback - Invoked as a response to TryInvokeTab() for SearchEdc
	/// </summary>
	/// <param name="SpawnTabArgs"></param>
	/// <returns></returns>
	TSharedRef<SDockTab> OnSpawnSearchTab(const class FSpawnTabArgs& SpawnTabArgs);

	/// <summary>
	/// Callback - Invoked as a response to RequestCloseTab() for OpenDocumentationWindow
	/// </summary>
	/// <param name="ClosedTab"></param>
	void HandleTutorialTabClosed(TSharedRef<SDockTab> ClosedTab);

	/// <summary>
	/// Callback - Invoked as a response to SWebBrowser::OnLoadError
	/// </summary>
	void HandleLoadErrorContent();

	/// <summary>
	/// Callback - Invoked as a response to SWebBrowser::OnBeforeNavigation
	/// </summary>
	/// <param name="NavUrl"></param>
	/// <param name="Nav"></param>
	/// <returns></returns>
	bool HandleEdcNavigation(const FString& NavUrl, const struct FWebNavigationRequest& Nav);

	/// <summary>
	/// Callback - Invoked as a response to SWebBrowser::OnLoadCompleted for the TutorialWebBrowser
	/// </summary>
	void HandleTutorialLoadCompleted();

	/// <summary>
	/// Callback - Invoked as a response to SWebBrowser::OnLoadCompleted for the SearchWebBrowser
	/// </summary>
	void HandleSearchLoadCompleted();

	/// <summary>
	/// Show the toast notification
	/// </summary>
	void DisplayTutorialToastNotification();

	/// <summary>
	/// Callback - Invoked as a response to Taost notification checkbox state changed
	/// </summary>
	/// <param name="NewState"></param>
	void HandleTutorialToastCheckboxStateChanged(ECheckBoxState NewState);

	/// <summary>
	/// Callback - Invoked as a response to module settings modified
	/// </summary>
	/// <returns></returns>
	bool HandleSettingsModified();
};
