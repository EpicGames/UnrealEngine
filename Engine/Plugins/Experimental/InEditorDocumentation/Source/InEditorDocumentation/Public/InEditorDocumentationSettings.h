// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "InEditorDocumentationSettings.generated.h"

/**
 * 
 */
UCLASS(Config = EditorPerProjectUserSettings, DefaultConfig)
class INEDITORDOCUMENTATION_API UInEditorDocumentationSettings : public UObject
{
	GENERATED_BODY()

public:

	/**
	UPROPERTY(EditAnywhere, Config, Category = "General", meta=(ToolTip="Default url to open if no search results found"))
	FString DefaultUrl = TEXT("https://dev.epicgames.com/documentation/unreal-engine");
	*/

	UPROPERTY(EditAnywhere, Config, Category = "Tutorial", meta=(ToolTip="Tutorial url to open if the tutorial button is clicked"))
	FString TutorialUrl = TEXT("https://dev.epicgames.com/documentation/unreal-engine/stack-o-bot-sample-game-in-unreal-engine");

	UPROPERTY(EditAnywhere, Config, Category = "Search (Experimental)", meta =(ToolTip="Enable Edc search for actors selected in the viewport"))
	bool bEnableEdcSearch = false;

	UPROPERTY(VisibleAnywhere, Config, Category = "Search (Experimental)", meta=(ToolTip="Api endpoint to search the edc for information about selected actor"))
	FString EdcSearchApiEndpoint = TEXT("https://dev.epicgames.com/community/api/search/index.json");

	UPROPERTY(EditAnywhere, Config, Category = "Search (Experimental)", meta=(ToolTip="List of predefined documentation page given an actor"))
	TMap<FString, FString> DocumentationPages;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};